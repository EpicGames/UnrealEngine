{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Apple Image Utils",
	"Description" : "Utilities that operate on CIImage, CVPixelBuffer, IOSurface, etc.",
	"Category" : "Experimental",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"SupportedPrograms": ["LiveLinkHub"],
	"Modules" :
	[
		{
			"Name" : "AppleImageUtils",
			"Type" : "Runtime",
			"LoadingPhase" : "Default",
			"ProgramAllowList": ["LiveLinkHub"]
		},
		{
			"Name" : "AppleImageUtilsBlueprintSupport",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Win64",
				"Mac",
				"Linux"
			]
		}
	]
}
