// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AppleImageUtils : ModuleRules
	{
		public AppleImageUtils(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "Engine",
                    // ... add other public dependencies that you statically link with here ...
                }
                );

    		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
    		{
                PublicFrameworks.AddRange(
                    new string[]
                    {
                        "CoreImage",
                        "ImageIO"
                        // ... add other public dependencies that you statically link with here ...
                    }
                    );
            }

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
    				"CoreUObject"
					// ... add private dependencies that you statically link with here ...
				}
				);

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					// ... add any modules that your module loads dynamically here ...
				}
				);
		}
	}
}
