// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleImageUtilsBlueprintProxy.h"
#include "IAppleImageUtilsPlugin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AppleImageUtilsBlueprintProxy)

UAppleImageUtilsBaseAsyncTaskBlueprintProxy::UAppleImageUtilsBaseAsyncTaskBlueprintProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bShouldTick(true)
{
}

void UAppleImageUtilsBaseAsyncTaskBlueprintProxy::Tick(float DeltaTime)
{
	if (!ConversionTask.IsValid())
	{
		ConversionResult.Error = TEXT("Invalid conversion task");
		bShouldTick = false;
		OnFailure.Broadcast(ConversionResult);
		return;
	}

	if (ConversionTask->IsDone())
	{
		bShouldTick = false;
		// Fire the right delegate
		if (!ConversionTask->HadError())
		{
			ConversionResult.Error = TEXT("Success");
			ConversionResult.ImageData = ConversionTask->GetData();
			OnSuccess.Broadcast(ConversionResult);
		}
		else
		{
			ConversionResult.Error = ConversionTask->GetErrorReason();
			OnFailure.Broadcast(ConversionResult);
		}
	}
}

UAppleImageUtilsBaseAsyncTaskBlueprintProxy* UAppleImageUtilsBaseAsyncTaskBlueprintProxy::CreateProxyObjectForConvertToJPEG(UTexture* SourceImage, int32 Quality, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate)
{
	UAppleImageUtilsBaseAsyncTaskBlueprintProxy* Proxy = NewObject<UAppleImageUtilsBaseAsyncTaskBlueprintProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);

	Proxy->ConversionTask = IAppleImageUtilsPlugin::Get().ConvertToJPEG(SourceImage, Quality, bWantColor, bUseGpu, Scale, Rotate);

	return Proxy;
}

UAppleImageUtilsBaseAsyncTaskBlueprintProxy* UAppleImageUtilsBaseAsyncTaskBlueprintProxy::CreateProxyObjectForConvertToHEIF(UTexture* SourceImage, int32 Quality, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate)
{
	UAppleImageUtilsBaseAsyncTaskBlueprintProxy* Proxy = NewObject<UAppleImageUtilsBaseAsyncTaskBlueprintProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);

	Proxy->ConversionTask = IAppleImageUtilsPlugin::Get().ConvertToHEIF(SourceImage, Quality, bWantColor, bUseGpu, Scale, Rotate);

	return Proxy;
}

UAppleImageUtilsBaseAsyncTaskBlueprintProxy* UAppleImageUtilsBaseAsyncTaskBlueprintProxy::CreateProxyObjectForConvertToTIFF(UTexture* SourceImage, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate)
{
	UAppleImageUtilsBaseAsyncTaskBlueprintProxy* Proxy = NewObject<UAppleImageUtilsBaseAsyncTaskBlueprintProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);

	Proxy->ConversionTask = IAppleImageUtilsPlugin::Get().ConvertToTIFF(SourceImage, bWantColor, bUseGpu, Scale, Rotate);

	return Proxy;
}

UAppleImageUtilsBaseAsyncTaskBlueprintProxy* UAppleImageUtilsBaseAsyncTaskBlueprintProxy::CreateProxyObjectForConvertToPNG(UTexture* SourceImage, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate)
{
	UAppleImageUtilsBaseAsyncTaskBlueprintProxy* Proxy = NewObject<UAppleImageUtilsBaseAsyncTaskBlueprintProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);

	Proxy->ConversionTask = IAppleImageUtilsPlugin::Get().ConvertToPNG(SourceImage, bWantColor, bUseGpu, Scale, Rotate);

	return Proxy;
}

