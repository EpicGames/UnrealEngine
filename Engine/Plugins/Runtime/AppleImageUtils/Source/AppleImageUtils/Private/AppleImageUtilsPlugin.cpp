// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAppleImageUtilsPlugin.h"
#include "Engine/Texture.h"

#if SUPPORTS_IMAGE_UTILS_1_0
	#include "Apple/ApplePlatformMisc.h"
	#include "Apple/ScopeAutoreleasePool.h"

#include "Async/Async.h"
	#include "Engine/Texture2D.h"

	#import <CoreImage/CIContext.h>

	// For runtime checks so that clang doesn't warn on targets < our SDK version
	#pragma clang diagnostic push
	#pragma clang diagnostic ignored "-Wpartial-availability"
#endif

class FAppleImageUtilsPlugin :
	public IAppleImageUtilsPlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> ConvertToJPEG(UTexture* SourceImage, int32 Quality = 85, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None) override;
	virtual TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> ConvertToHEIF(UTexture* SourceImage, int32 Quality = 85, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None) override;
	virtual TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> ConvertToPNG(UTexture* SourceImage, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None) override;
	virtual TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> ConvertToTIFF(UTexture* SourceImage, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None) override;
#if SUPPORTS_IMAGE_UTILS_1_0
	virtual CGImageRef UTexture2DToCGImage(UTexture2D* Source) override;
	virtual void ConvertToJPEG(CIImage* SourceImage, TArray<uint8>& OutBytes, int32 Quality = 85, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None, FIntPoint* ConvertedImageSize = nullptr) override;
#if SUPPORTS_IMAGE_UTILS_2_1
	virtual void ConvertToHEIF(CIImage* SourceImage, TArray<uint8>& OutBytes, int32 Quality = 85,  bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None, FIntPoint* ConvertedImageSize = nullptr) override;
	virtual void ConvertToPNG(CIImage* SourceImage, TArray<uint8>& OutBytes, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None, FIntPoint* ConvertedImageSize = nullptr) override;
	virtual void ConvertToTIFF(CIImage* SourceImage, TArray<uint8>& OutBytes, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None, FIntPoint* ConvertedImageSize = nullptr) override;
#endif
#endif
};

IMPLEMENT_MODULE(FAppleImageUtilsPlugin, AppleImageUtils);

void FAppleImageUtilsPlugin::StartupModule()
{
}

void FAppleImageUtilsPlugin::ShutdownModule()
{
}

class FAppleImageUtilsFailedConversionTask :
	public FAppleImageUtilsConversionTaskBase
{
public:
	FAppleImageUtilsFailedConversionTask(const FString& InError) :
		FAppleImageUtilsConversionTaskBase()
	{
		Error = InError;
		bHadError = true;
		bIsDone = true;
	}

	virtual TArray<uint8> GetData() override { return TArray<uint8>(); }
};

class FAppleImageUtilsConversionTask :
	public FAppleImageUtilsConversionTaskBase
{
public:
#if SUPPORTS_IMAGE_UTILS_1_0
	FAppleImageUtilsConversionTask(CIImage* InSourceImage) :
		FAppleImageUtilsConversionTaskBase()
		, SourceImage(InSourceImage)
	{
		check(InSourceImage != nullptr);
		StartTime = FPlatformTime::Seconds();
	}

	virtual ~FAppleImageUtilsConversionTask() { }
#else
	FAppleImageUtilsConversionTask() :
		FAppleImageUtilsConversionTaskBase()
	{
	}
#endif

	//~ IAppleImageUtilsConversionTask
	virtual TArray<uint8> GetData() override { return MoveTemp(ConvertedBytes); }
	virtual FIntPoint GetConvertedImageSize() const override { return ConvertedImageSize; };
	//~ IAppleImageUtilsConversionTask

	void MarkComplete()
	{
#if SUPPORTS_IMAGE_UTILS_1_0
		// Release on completion since we don't need it anymore
		if (SourceImage != nullptr)
		{
			[SourceImage release];
			SourceImage = nullptr;
		}
#endif
		EndTime = FPlatformTime::Seconds();
		bIsDone = true;
	}

#if SUPPORTS_IMAGE_UTILS_1_0
	/** Must be released */
	CIImage* SourceImage;
#endif
	/** Where the data is placed when the task is done */
	TArray<uint8> ConvertedBytes;
	
	/** The size of the converted image */
	FIntPoint ConvertedImageSize = FIntPoint::ZeroValue;
};

#if SUPPORTS_IMAGE_UTILS_1_0
static inline NSDictionary* ToQualityDictionary(int32 Quality)
{
	// Our code uses ints but Apple uses floats 0..1
	Quality = FMath::Clamp(Quality, 0, 100);
	float QualityF = float(Quality) * 0.01f;

	NSDictionary* Options = [NSDictionary dictionaryWithObjectsAndKeys:[NSNumber numberWithFloat: QualityF], kCGImageDestinationLossyCompressionQuality, nil];
	return Options;
}

static inline NSDictionary* ToCpuDictionary(bool bUseGpu)
{
	BOOL UseCpu = bUseGpu ? NO : YES;
	NSDictionary* Options = [NSDictionary dictionaryWithObjectsAndKeys:[NSNumber numberWithBool: UseCpu], kCIContextUseSoftwareRenderer, nil];
	return Options;
}

static inline CGColorSpaceRef ToColorSpace(bool bWantColor)
{
	return bWantColor ? CGColorSpaceCreateWithName(kCGColorSpaceSRGB) : CGColorSpaceCreateWithName(kCGColorSpaceGenericGrayGamma2_2);
}

/** Note this image object must be released */
static inline CIImage* AllocateImage(IAppleImageInterface* AppleImageInterface)
{
	// Touching UObjects so must be in game thread, hence not part of the autorelease pool due to multithreading
	check(IsInGameThread());
	check(AppleImageInterface != nullptr);

	CIImage* Image = nullptr;
	switch (AppleImageInterface->GetTextureType())
	{
		case EAppleTextureType::Image:
		{
			Image = AppleImageInterface->GetImage();
			[Image retain];
			break;
		}

		case EAppleTextureType::PixelBuffer:
		{
			CVPixelBufferRef PixelBuffer = AppleImageInterface->GetPixelBuffer();
			if (PixelBuffer != nullptr)
			{
				Image = [[CIImage alloc] initWithCVPixelBuffer: PixelBuffer];
			}
			break;
		}

		case EAppleTextureType::Surface:
		{
			IOSurfaceRef Surface = AppleImageInterface->GetSurface();
			if (Surface != nullptr)
			{
				Image = [[CIImage alloc] initWithIOSurface: Surface];
			}
			break;
		}
			
		case EAppleTextureType::MetalTexture:
		{
			if (auto MetalTexture = AppleImageInterface->GetMetalTexture())
			{
				// initWithMTLTexture seems to initialize the image as up-side-down flipped
				// and we need to fix it before the conversion
				CIImage* MetalImage = [[CIImage alloc] initWithMTLTexture: MetalTexture options: nullptr];
				Image = [MetalImage imageByApplyingOrientation: kCGImagePropertyOrientationDownMirrored];
				[Image retain];
				[MetalImage release];
			}
			break;
		}
	}
	return Image;
}

/** Note must be called from the processing thread since this uses the autorelease pool and assumes scoped release pools */
static inline CIImage* ApplyScaleAndRotation(CIImage* SourceImage, float Scale, ETextureRotationDirection Rotate)
{
	CIImage* Image = SourceImage;
	// Handle scaling if requested
	if (Scale != 1.f)
	{
		CGRect Rect = Image.extent;
		// Note that this is the additional aspect ratio to apply, which we don't need
		// See https://developer.apple.com/documentation/coreimage/cilanczosscaletransform/3228516-aspectratio?language=objc
		const auto AspectRatio = 1.f;
		CIFilter* ScaleFilter = [CIFilter filterWithName: @"CILanczosScaleTransform"];
		[ScaleFilter setValue: Image forKey: kCIInputImageKey];
		[ScaleFilter setValue: @(Scale) forKey: kCIInputScaleKey];
		[ScaleFilter setValue: @(AspectRatio) forKey: kCIInputAspectRatioKey];
		Image = ScaleFilter.outputImage;
	}
	// Handle rotation if requested
	switch (Rotate)
	{
		case ETextureRotationDirection::Left:
			Image = [Image imageByApplyingOrientation: kCGImagePropertyOrientationLeft];
			break;
		case ETextureRotationDirection::Right:
			Image = [Image imageByApplyingOrientation: kCGImagePropertyOrientationRight];
			break;
		case ETextureRotationDirection::Down:
			Image = [Image imageByApplyingOrientation: kCGImagePropertyOrientationDown];
			break;
		case ETextureRotationDirection::LeftMirrored:
			Image = [Image imageByApplyingOrientation: kCGImagePropertyOrientationLeftMirrored];
			break;
		case ETextureRotationDirection::RightMirrored:
			Image = [Image imageByApplyingOrientation: kCGImagePropertyOrientationRightMirrored];
			break;
		case ETextureRotationDirection::DownMirrored:
			Image = [Image imageByApplyingOrientation: kCGImagePropertyOrientationDownMirrored];
			break;
		case ETextureRotationDirection::UpMirrored:
			Image = [Image imageByApplyingOrientation: kCGImagePropertyOrientationUpMirrored];
			break;
	}
	return Image;
}
#endif

static inline bool CanBeConverted(IAppleImageInterface* AppleImageInterface)
{
	check(AppleImageInterface != nullptr);

	switch (AppleImageInterface->GetTextureType())
	{
		case EAppleTextureType::Image:
		case EAppleTextureType::PixelBuffer:
		case EAppleTextureType::Surface:
		case EAppleTextureType::MetalTexture:
			return true;
	}
	return false;
}

TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> FAppleImageUtilsPlugin::ConvertToJPEG(UTexture* SourceImage, int32 Quality, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate)
{
	// Make sure our interface is supported
	IAppleImageInterface* AppleImage = Cast<IAppleImageInterface>(SourceImage);
	if (AppleImage == nullptr)
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToJPEG only supports UAppleImageInterface derived textures"));
	}
	if (!CanBeConverted(AppleImage))
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(FString::Printf(TEXT("ConvertToJPEG texture type (%d) was not supported"), (int32)AppleImage->GetTextureType()));
	}

#if SUPPORTS_IMAGE_UTILS_1_0
	if (!FAppleImageUtilsAvailability::SupportsImageUtils10())
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToJPEG requires iOS 10.0+ or macOS 10.12+"));
	}

	TSharedPtr<FAppleImageUtilsConversionTask, ESPMode::ThreadSafe> ConversionTask = MakeShared<FAppleImageUtilsConversionTask, ESPMode::ThreadSafe>(AllocateImage(AppleImage));
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, ConversionTask, Quality, bWantColor, bUseGpu, Scale, Rotate]()
	{
		ConvertToJPEG(ConversionTask->SourceImage, ConversionTask->ConvertedBytes, Quality, bWantColor, bUseGpu, Scale, Rotate, &ConversionTask->ConvertedImageSize);
		// Notify any async listeners that we are done
		ConversionTask->MarkComplete();
	});
	return ConversionTask;
#else
	return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToJPEG requires iOS 10.0+ or macOS 10.12+"));
#endif
}

TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> FAppleImageUtilsPlugin::ConvertToHEIF(UTexture* SourceImage, int32 Quality, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate)
{
	// Make sure our interface is supported
	IAppleImageInterface* AppleImage = Cast<IAppleImageInterface>(SourceImage);
	if (AppleImage == nullptr)
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToHEIF only supports UAppleImageInterface derived textures"));
	}
	if (!CanBeConverted(AppleImage))
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(FString::Printf(TEXT("ConvertToHEIF texture type (%d) was not supported"), (int32)AppleImage->GetTextureType()));
	}

#if SUPPORTS_IMAGE_UTILS_2_1
	if (!FAppleImageUtilsAvailability::SupportsImageUtils21())
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToHEIF requires iOS 11.0+ or macOS 10.13.4+"));
	}

	TSharedPtr<FAppleImageUtilsConversionTask, ESPMode::ThreadSafe> ConversionTask = MakeShared<FAppleImageUtilsConversionTask, ESPMode::ThreadSafe>(AllocateImage(AppleImage));
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, ConversionTask, Quality, bWantColor, bUseGpu, Scale, Rotate]()
	{
		ConvertToHEIF(ConversionTask->SourceImage, ConversionTask->ConvertedBytes, Quality, bWantColor, bUseGpu, Scale, Rotate, &ConversionTask->ConvertedImageSize);
		// Notify any async listeners that we are done
		ConversionTask->MarkComplete();
	});
	return ConversionTask;
#else
	return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToHEIF requires iOS 11.0+ or macOS 10.13.4+"));
#endif
}

TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> FAppleImageUtilsPlugin::ConvertToPNG(UTexture* SourceImage, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate)
{
	// Make sure our interface is supported
	IAppleImageInterface* AppleImage = Cast<IAppleImageInterface>(SourceImage);
	if (AppleImage == nullptr)
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToPNG only supports UAppleImageInterface derived textures"));
	}
	if (!CanBeConverted(AppleImage))
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(FString::Printf(TEXT("ConvertToPNG texture type (%d) was not supported"), (int32)AppleImage->GetTextureType()));
	}

#if SUPPORTS_IMAGE_UTILS_2_1
	if (!FAppleImageUtilsAvailability::SupportsImageUtils21())
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToPNG requires iOS 11.0+ or macOS 10.13.4+"));
	}

	TSharedPtr<FAppleImageUtilsConversionTask, ESPMode::ThreadSafe> ConversionTask = MakeShared<FAppleImageUtilsConversionTask, ESPMode::ThreadSafe>(AllocateImage(AppleImage));
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, ConversionTask, bWantColor, bUseGpu, Scale, Rotate]()
	{
		ConvertToPNG(ConversionTask->SourceImage, ConversionTask->ConvertedBytes, bWantColor, bUseGpu, Scale, Rotate, &ConversionTask->ConvertedImageSize);
		// Notify any async listeners that we are done
		ConversionTask->MarkComplete();
	});
	return ConversionTask;
#else
	return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToPNG requires iOS 11.0+ or macOS 10.13.4+"));
#endif
}

TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> FAppleImageUtilsPlugin::ConvertToTIFF(UTexture* SourceImage, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate)
{
	// Make sure our interface is supported
	IAppleImageInterface* AppleImage = Cast<IAppleImageInterface>(SourceImage);
	if (AppleImage == nullptr)
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToTIFF only supports UAppleImageInterface derived textures"));
	}
	if (!CanBeConverted(AppleImage))
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(FString::Printf(TEXT("ConvertToTIFF texture type (%d) was not supported"), (int32)AppleImage->GetTextureType()));
	}

#if SUPPORTS_IMAGE_UTILS_2_1
	if (!FAppleImageUtilsAvailability::SupportsImageUtils21())
	{
		return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToTIFF requires iOS 11.0+ or macOS 10.13.4+"));
	}

	TSharedPtr<FAppleImageUtilsConversionTask, ESPMode::ThreadSafe> ConversionTask = MakeShared<FAppleImageUtilsConversionTask, ESPMode::ThreadSafe>(AllocateImage(AppleImage));
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, ConversionTask, bWantColor, bUseGpu, Scale, Rotate]()
	{
		ConvertToTIFF(ConversionTask->SourceImage, ConversionTask->ConvertedBytes, bWantColor, bUseGpu, Scale, Rotate, &ConversionTask->ConvertedImageSize);
		// Notify any async listeners that we are done
		ConversionTask->MarkComplete();
	});
	return ConversionTask;
#else
	return MakeShared<FAppleImageUtilsFailedConversionTask, ESPMode::ThreadSafe>(TEXT("ConvertToTIFF requires iOS 11.0+ or macOS 10.13.4+"));
#endif
}

#if SUPPORTS_IMAGE_UTILS_1_0

CGImageRef FAppleImageUtilsPlugin::UTexture2DToCGImage(UTexture2D* Source)
{
	check(Source != nullptr);

	const EPixelFormat SourceFormat = Source->GetPixelFormat();
	if (!(SourceFormat == PF_A8R8G8B8 || SourceFormat == PF_R8G8B8A8 || SourceFormat == PF_B8G8R8A8))
	{
		UE_LOGF(LogTemp, Warning, "TextureToCGImage() can only convert textures of types PF_A8R8G8B8, PF_R8G8B8A8, and PF_B8G8R8A8");
		return nullptr;
	}

	int32 Width = Source->GetSizeX();
	int32 Height = Source->GetSizeY();
	const int32 NumComponents = 4;
	const int32 BitsPerComponent = 8;
	const int32 BitsPerPixel = NumComponents * BitsPerComponent;
	const int32 ImageSizeInBytes = Width * Height * NumComponents;
	const int32 BytesPerRow = Width * NumComponents;

    CGBitmapInfo BitmapInfo = 0;
	switch (SourceFormat)
	{
		case PF_A8R8G8B8:
		{
            BitmapInfo = kCGBitmapByteOrder32Big | kCGImageAlphaLast;
			break;
		}
        case PF_R8G8B8A8:
        {
            BitmapInfo = kCGBitmapByteOrder32Big | kCGImageAlphaFirst;
            break;
        }
        case PF_B8G8R8A8:
        {
            BitmapInfo = kCGBitmapByteOrder32Little | kCGImageAlphaLast;
            break;
        }
	}

	CGImageRef ImageRef = nullptr;

	TArray<uint8*> MipPointers;
	MipPointers.AddZeroed(Source->GetNumMips());
	Source->GetMipData(0, (void**)MipPointers.GetData());
	if (MipPointers[0] != nullptr)
	{
        // The data ref is going to free this result
		CFDataRef DataRef = CFDataCreateWithBytesNoCopy(NULL, MipPointers[0], ImageSizeInBytes, kCFAllocatorDefault);
		CGDataProviderRef DataProviderRef = CGDataProviderCreateWithCFData(DataRef);
		CGColorSpaceRef ColorSpaceRef = CGColorSpaceCreateDeviceRGB();

		ImageRef = CGImageCreate(Width, Height, BitsPerComponent, BitsPerPixel, BytesPerRow, ColorSpaceRef, BitmapInfo, DataProviderRef, nullptr, true, kCGRenderingIntentDefault);

		// Release our temporary memory
		CGColorSpaceRelease(ColorSpaceRef);
		CGDataProviderRelease(DataProviderRef);
		CFRelease(DataRef);
	}
	return ImageRef;
}

void FAppleImageUtilsPlugin::ConvertToJPEG(CIImage* SourceImage, TArray<uint8>& OutBytes, int32 Quality, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate, FIntPoint* ConvertedImageSize)
{
	SCOPED_AUTORELEASE_POOL;

	// Convert to Apple objects
	CIContext* ConversionContext = [CIContext contextWithOptions: ToCpuDictionary(bUseGpu)];
	CGColorSpaceRef ColorSpace = ToColorSpace(bWantColor);
	CIImage* Image = ApplyScaleAndRotation(SourceImage, Scale, Rotate);
	
	if (ConvertedImageSize)
	{
		ConvertedImageSize->X = (int32)Image.extent.size.width;
		ConvertedImageSize->Y = (int32)Image.extent.size.height;
	}

	// This will perform the work on the GPU or inline on this thread
	NSData* ConvertedData = [ConversionContext JPEGRepresentationOfImage: Image colorSpace: ColorSpace options: ToQualityDictionary(Quality)];
	if (ConvertedData != nullptr)
	{
		uint32 CompressedSize = ConvertedData.length;
		OutBytes.AddUninitialized(CompressedSize);
		FPlatformMemory::Memcpy(OutBytes.GetData(), [ConvertedData bytes], CompressedSize);
	}

	CGColorSpaceRelease(ColorSpace);
}

#if SUPPORTS_IMAGE_UTILS_2_1
void FAppleImageUtilsPlugin::ConvertToHEIF(CIImage* SourceImage, TArray<uint8>& OutBytes, int32 Quality,  bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate, FIntPoint* ConvertedImageSize)
{
	SCOPED_AUTORELEASE_POOL;

	// Convert to Apple objects
	CIContext* ConversionContext = [CIContext contextWithOptions: ToCpuDictionary(bUseGpu)];
	CGColorSpaceRef ColorSpace = ToColorSpace(bWantColor);
	CIImage* Image = ApplyScaleAndRotation(SourceImage, Scale, Rotate);
	
	if (ConvertedImageSize)
	{
		ConvertedImageSize->X = (int32)Image.extent.size.width;
		ConvertedImageSize->Y = (int32)Image.extent.size.height;
	}

	// This will perform the work on the GPU or inline on this thread
	NSData* ConvertedData = [ConversionContext HEIFRepresentationOfImage: Image format: kCIFormatARGB8 colorSpace: ColorSpace options: ToQualityDictionary(Quality)];
	if (ConvertedData != nullptr)
	{
		uint32 CompressedSize = ConvertedData.length;
		OutBytes.AddUninitialized(CompressedSize);
		FPlatformMemory::Memcpy(OutBytes.GetData(), [ConvertedData bytes], CompressedSize);
	}

	CGColorSpaceRelease(ColorSpace);
}

void FAppleImageUtilsPlugin::ConvertToPNG(CIImage* SourceImage, TArray<uint8>& OutBytes, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate, FIntPoint* ConvertedImageSize)
{
	SCOPED_AUTORELEASE_POOL;

	// Convert to Apple objects
	CIContext* ConversionContext = [CIContext contextWithOptions: ToCpuDictionary(bUseGpu)];
	CGColorSpaceRef ColorSpace = ToColorSpace(bWantColor);
	CIImage* Image = ApplyScaleAndRotation(SourceImage, Scale, Rotate);
	
	if (ConvertedImageSize)
	{
		ConvertedImageSize->X = (int32)Image.extent.size.width;
		ConvertedImageSize->Y = (int32)Image.extent.size.height;
	}

	// This will perform the work on the GPU or inline on this thread
	NSData* ConvertedData = [ConversionContext PNGRepresentationOfImage: Image format: kCIFormatARGB8 colorSpace: ColorSpace options: @{}];
	if (ConvertedData != nullptr)
	{
		uint32 CompressedSize = ConvertedData.length;
		OutBytes.AddUninitialized(CompressedSize);
		FPlatformMemory::Memcpy(OutBytes.GetData(), [ConvertedData bytes], CompressedSize);
	}

	CGColorSpaceRelease(ColorSpace);
}

void FAppleImageUtilsPlugin::ConvertToTIFF(CIImage* SourceImage, TArray<uint8>& OutBytes, bool bWantColor, bool bUseGpu, float Scale, ETextureRotationDirection Rotate, FIntPoint* ConvertedImageSize)
{
	SCOPED_AUTORELEASE_POOL;

	// Convert to Apple objects
	CIContext* ConversionContext = [CIContext contextWithOptions: ToCpuDictionary(bUseGpu)];
	CGColorSpaceRef ColorSpace = ToColorSpace(bWantColor);
	CIImage* Image = ApplyScaleAndRotation(SourceImage, Scale, Rotate);
	
	if (ConvertedImageSize)
	{
		ConvertedImageSize->X = (int32)Image.extent.size.width;
		ConvertedImageSize->Y = (int32)Image.extent.size.height;
	}

	// This will perform the work on the GPU or inline on this thread
	NSData* ConvertedData = [ConversionContext TIFFRepresentationOfImage: Image format: kCIFormatARGB8 colorSpace: ColorSpace options: @{}];
	if (ConvertedData != nullptr)
	{
		uint32 CompressedSize = ConvertedData.length;
		OutBytes.AddUninitialized(CompressedSize);
		FPlatformMemory::Memcpy(OutBytes.GetData(), [ConvertedData bytes], CompressedSize);
	}

	CGColorSpaceRelease(ColorSpace);
}
#endif

	#pragma clang diagnostic pop
#endif

/** Dummy class needed to support Cast<IAppleImageInterface>(Object) */
UAppleImageInterface::UAppleImageInterface(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
}
