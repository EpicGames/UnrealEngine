// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef PLATFORM_IOS
	#define PLATFORM_IOS 0
#endif

#ifndef PLATFORM_TVOS
	#define PLATFORM_TVOS 0
#endif

#ifndef PLATFORM_MAC
	#define PLATFORM_MAC 0
#endif

#if PLATFORM_IOS && !PLATFORM_TVOS

	// Check for image utils 1.0
	#ifdef __IPHONE_10_0
		#define SUPPORTS_IMAGE_UTILS_1_0 1
		#define RT_CHECK_IMAGE_UTILS_1_0 @available(iOS 10.0, *)
	#else
		#define SUPPORTS_IMAGE_UTILS_1_0 0
	#endif

	// Check for image utils 2
	#ifdef __IPHONE_11_0
		#define SUPPORTS_IMAGE_UTILS_2_0 1
		#define RT_CHECK_IMAGE_UTILS_2_0 @available(iOS 11.0, *)
	#else
		#define SUPPORTS_IMAGE_UTILS_2_0 0
	#endif

	// Check for image utils 2.1
	#ifdef __IPHONE_11_0
		#define SUPPORTS_IMAGE_UTILS_2_1 1
		#define RT_CHECK_IMAGE_UTILS_2_1 @available(iOS 11.0, *)
	#else
		#define SUPPORTS_IMAGE_UTILS_2_1 0
	#endif

#elif PLATFORM_MAC

	// Check for image utils 1.0
	#ifdef MAC_OS_X_VERSION_10_12
		#define SUPPORTS_IMAGE_UTILS_1_0 1
		#define RT_CHECK_IMAGE_UTILS_1_0 @available(macOS 10.12, *)
	#else
		#define SUPPORTS_IMAGE_UTILS_1_0 0
	#endif

	// Check for image utils 2
	#ifdef MAC_OS_X_VERSION_10_13
		#define SUPPORTS_IMAGE_UTILS_2_0 1
		#define RT_CHECK_IMAGE_UTILS_2_0 @available(macOS 10.13, *)
	#else
		#define SUPPORTS_IMAGE_UTILS_2_0 0
	#endif

	// Check for image utils 2.1
	#ifdef MAC_OS_X_VERSION_10_13
        //@todo joeg -- enable again once you figure out the failure on 10.12.6
		#define SUPPORTS_IMAGE_UTILS_2_1 0
		#define RT_CHECK_IMAGE_UTILS_2_1 @available(macOS 10.13.4, *)
	#else
		#define SUPPORTS_IMAGE_UTILS_2_1 0
	#endif

#else

	#define SUPPORTS_IMAGE_UTILS_1_0 0
	#define SUPPORTS_IMAGE_UTILS_2_0 0
	#define SUPPORTS_IMAGE_UTILS_2_1 0

#endif

class FAppleImageUtilsAvailability
{
public:
	static bool SupportsImageUtils10()
	{
		static bool bSupportsImageUtils10 = false;
#if SUPPORTS_IMAGE_UTILS_1_0
		static bool bSupportChecked = false;
		if (!bSupportChecked)
		{
			bSupportChecked = true;
			// This call is slow, so cache the result
			if (RT_CHECK_IMAGE_UTILS_1_0)
			{
				bSupportsImageUtils10 = true;
			}
		}
#endif
		return bSupportsImageUtils10;
	}

	static bool SupportsImageUtils20()
	{
		static bool bSupportsImageUtils20 = false;
#if SUPPORTS_IMAGE_UTILS_2_0
		static bool bSupportChecked = false;
		if (!bSupportChecked)
		{
			bSupportChecked = true;
			// This call is slow, so cache the result
			if (RT_CHECK_IMAGE_UTILS_2_0)
			{
				bSupportsImageUtils20 = true;
			}
		}
#endif
		return bSupportsImageUtils20;
	}

	static bool SupportsImageUtils21()
	{
		static bool bSupportsImageUtils21 = false;
#if SUPPORTS_IMAGE_UTILS_2_1
		static bool bSupportChecked = false;
		if (!bSupportChecked)
		{
			bSupportChecked = true;
			// This call is slow, so cache the result
			if (RT_CHECK_IMAGE_UTILS_2_1)
			{
				bSupportsImageUtils21 = true;
			}
		}
#endif
		return bSupportsImageUtils21;
	}
};
