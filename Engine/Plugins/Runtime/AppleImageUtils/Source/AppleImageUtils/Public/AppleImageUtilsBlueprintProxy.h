// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tickable.h"

#include "AppleImageUtilsTypes.h"

#include "AppleImageUtilsBlueprintProxy.generated.h"

class UTexture;

class FAppleImageUtilsConversionTaskBase;

USTRUCT(BlueprintType)
struct FAppleImageUtilsImageConversionResult
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadOnly, Category="Image Conversion")
	FString Error;

	UPROPERTY(BlueprintReadOnly, Category="Image Conversion")
	TArray<uint8> ImageData;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FAppleImageConversionDelegate, const FAppleImageUtilsImageConversionResult&, ConversionResult);

UCLASS(MinimalAPI)
class UAppleImageUtilsBaseAsyncTaskBlueprintProxy :
	public UObject,
	public FTickableGameObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(BlueprintAssignable)
	FAppleImageConversionDelegate OnSuccess;

	UPROPERTY(BlueprintAssignable)
	FAppleImageConversionDelegate OnFailure;

	/**
	 * Converts a image to an array of JPEG data in a background task
	 *
	 * @param SourceImage the image to compress
	 * @param Quality the quality level to compress to
	 * @param bWantColor whether the JPEG is color (true) or monochrome (false)
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before conversion, defaults to no scaling
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 */
	UFUNCTION(BlueprintCallable, Meta=(BlueprintInternalUseOnly="true", DisplayName="Convert To JPEG"), Category="Image Conversion")
	static UAppleImageUtilsBaseAsyncTaskBlueprintProxy* CreateProxyObjectForConvertToJPEG(UTexture* SourceImage, int32 Quality = 85, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None);

	/**
	 * Converts a image to an array of HEIF data in a background task
	 *
	 * @param SourceImage the image to compress
	 * @param Quality the quality level to compress to
	 * @param bWantColor whether the HEIF is color (true) or monochrome (false)
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before conversion, defaults to no scaling
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 */
	UFUNCTION(BlueprintCallable, Meta=(BlueprintInternalUseOnly="true", DisplayName="Convert To HEIF"), Category="Image Conversion")
	static UAppleImageUtilsBaseAsyncTaskBlueprintProxy* CreateProxyObjectForConvertToHEIF(UTexture* SourceImage, int32 Quality = 85, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None);

	/**
	 * Converts a image to an array of TIFF data in a background task
	 *
	 * @param SourceImage the image to compress
	 * @param Quality the quality level to compress to
	 * @param bWantColor whether the TIFF is color (true) or monochrome (false)
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before conversion, defaults to no scaling
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 */
	UFUNCTION(BlueprintCallable, Meta=(BlueprintInternalUseOnly="true", DisplayName="Convert To TIFF"), Category="Image Conversion")
	static UAppleImageUtilsBaseAsyncTaskBlueprintProxy* CreateProxyObjectForConvertToTIFF(UTexture* SourceImage, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None);

	/**
	 * Converts a image to an array of TIFF data in a background task
	 *
	 * @param SourceImage the image to compress
	 * @param Quality the quality level to compress to
	 * @param bWantColor whether the PNG is color (true) or monochrome (false)
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before conversion, defaults to no scaling
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 */
	UFUNCTION(BlueprintCallable, Meta=(BlueprintInternalUseOnly="true", DisplayName="Convert To PNG"), Category="Image Conversion")
	static UAppleImageUtilsBaseAsyncTaskBlueprintProxy* CreateProxyObjectForConvertToPNG(UTexture* SourceImage, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None);

	//~ Begin FTickableObject Interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return bShouldTick; }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UAppleImageUtilsBaseAsyncTaskBlueprintProxy, STATGROUP_Tickables); }
	//~ End FTickableObject Interface

	/** The async task to check during Tick() */
	TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> ConversionTask;

	UPROPERTY(BlueprintReadOnly, Category="Image Conversion")
	FAppleImageUtilsImageConversionResult ConversionResult;

private:
	/** True until the async task completes, then false */
	bool bShouldTick;
};
