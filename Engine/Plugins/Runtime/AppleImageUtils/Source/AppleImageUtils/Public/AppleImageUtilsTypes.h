// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif
#if PLATFORM_MAC || PLATFORM_IOS
	#import <CoreVideo/CoreVideo.h>
	#import <CoreImage/CoreImage.h>
#endif
#include "AppleImageUtilsTypes.generated.h"

UENUM(BlueprintType, Category="AppleImageUtils", meta=(Experimental))
enum class ETextureRotationDirection : uint8
{
	None,
	Left,
	Right,
	Down,
	LeftMirrored,
	RightMirrored,
	DownMirrored,
	UpMirrored
};

UENUM(BlueprintType, Category="AppleImageUtils", meta=(Experimental))
enum class EAppleTextureType : uint8
{
	Unknown,
	Image,
	PixelBuffer,
	Surface,
	MetalTexture
};

/** Dummy class needed to support Cast<IAppleImageInterface>(Object). */
UINTERFACE(MinimalAPI)
class UAppleImageInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/**
 * Base class for accessing the raw Apple image data
 */
class IAppleImageInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	/** @return the type of image held by the implementing object */
	virtual EAppleTextureType GetTextureType() const = 0;

#if PLATFORM_MAC || PLATFORM_IOS
	/** @return the CIImage held by the implementing object */
	virtual CIImage* GetImage() const { return nullptr; }
	/** @return the CVPixelBuffer held by the implementing object */
	virtual CVPixelBufferRef GetPixelBuffer() const { return nullptr; }
	/** @return the IOSurface held by the implementing object */
	virtual IOSurfaceRef GetSurface() const { return nullptr; }

	/** @return the MTLTextureid<MTL held by the implementing object */
	virtual id<MTLTexture> GetMetalTexture() const { return nullptr; }
#endif
};
