// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "HAL/ThreadSafeBool.h"

#include "AppleImageUtilsTypes.h"

#include "AppleImageUtilsAvailability.h"

#if SUPPORTS_IMAGE_UTILS_1_0
	#import <CoreGraphics/CoreGraphics.h>
#endif

class UTexture;
class UTexture2D;

/**
 * Interface to access the async compression request
 */
class IAppleImageUtilsConversionTask
{
public:
	/** @return whether the task succeeded or not */
	virtual bool HadError() const = 0;
	/** @return information about the error if there was one */
	virtual FString GetErrorReason() const = 0;
	/** @return whether the task has completed or not */
	virtual bool IsDone() const = 0;
	/** @return how long it took this task to complete */
	virtual float GetElapsedTime() const = 0;
	/** @return the data once the task has completed or empty array if still in progress. NOTE: this should use MoveTemp() to avoid duplication so call once :) */
	virtual TArray<uint8> GetData() = 0;
	/** @return the size of the converted image */
	virtual FIntPoint GetConvertedImageSize() const = 0;
};

/**
 * Base class for implementing IAppleImageUtilsCompressionTasks
 */
class FAppleImageUtilsConversionTaskBase :
	public IAppleImageUtilsConversionTask
{
protected:
	FAppleImageUtilsConversionTaskBase() {}
	virtual ~FAppleImageUtilsConversionTaskBase() = default;

public:
	virtual bool IsDone() const override { return bIsDone; }
	virtual bool HadError() const override { return bHadError; }
	virtual FString GetErrorReason() const override { return Error; }
	virtual float GetElapsedTime() const override { return (float)(EndTime - StartTime) * 1000.f; }
	virtual FIntPoint GetConvertedImageSize() const override { return FIntPoint::ZeroValue; };

protected:
	FThreadSafeBool bIsDone;
	FThreadSafeBool bHadError;
	double StartTime = 0.f;
	double EndTime = 0.f;
	FString Error;
};

class IAppleImageUtilsPlugin :
	public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IAppleImageUtilsPlugin& Load()
	{
		return FModuleManager::LoadModuleChecked<IAppleImageUtilsPlugin>("AppleImageUtils");
	}

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance
	 */
	static inline IAppleImageUtilsPlugin& Get()
	{
		return FModuleManager::GetModuleChecked<IAppleImageUtilsPlugin>("AppleImageUtils");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("AppleImageUtils");
	}

	/**
	 * Converts a image to an array of JPEG data in a background task
	 *
	 * @param SourceImage the image to compress (NOTE: must support UAppleImageInterface)
	 * @param Quality the quality level to compress to
	 * @param bWantColor whether the JPEG is color (true) or monochrome (false)
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before converting, defaults to no scale operation
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 *
	 * @return the async task that is doing the conversion
	 */
	virtual TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> ConvertToJPEG(UTexture* SourceImage, int32 Quality = 85, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None) = 0;

	/**
	 * Converts a image to an array of HEIF data in a background task
	 *
	 * @param SourceImage the image to compress (NOTE: must support UAppleImageInterface)
	 * @param Quality the quality level to compress to
	 * @param bWantColor whether the HEIF is color (true) or monochrome (false)
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before converting, defaults to no scale operation
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 *
	 * @return the async task that is doing the conversion
	 */
	virtual TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> ConvertToHEIF(UTexture* SourceImage, int32 Quality = 85,  bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None) = 0;

	/**
	 * Converts a image to an array of PNG data in a background task
	 *
	 * @param SourceImage the image to compress (NOTE: must support UAppleImageInterface)
	 * @param Quality the quality level to compress to
	 * @param bWantColor whether the PNG is color (true) or monochrome (false)
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before converting, defaults to no scale operation
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 *
	 * @return the async task that is doing the conversion
	 */
	virtual TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> ConvertToPNG(UTexture* SourceImage, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None) = 0;

	/**
	 * Converts a image to an array of TIFF data in a background task
	 *
	 * @param SourceImage the image to compress (NOTE: must support UAppleImageInterface)
	 * @param Quality the quality level to compress to
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before converting, defaults to no scale operation
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 *
	 * @return the async task that is doing the conversion
	 */
	virtual TSharedPtr<FAppleImageUtilsConversionTaskBase, ESPMode::ThreadSafe> ConvertToTIFF(UTexture* SourceImage, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None) = 0;

#if SUPPORTS_IMAGE_UTILS_1_0
	/**
	 * Copies the contents of a UTexture2D to a CGImage object
	 *
	 * @param SourceImage the image to create a CGImage from
	 *
	 * @return the CGImage if conversion was possible
	 */
	virtual CGImageRef UTexture2DToCGImage(UTexture2D* Source) = 0;

	/**
	 * Converts a image to an array of JPEG data synchronously
	 *
	 * @param SourceImage the image to compress
	 * @param OutBytes the buffer that is populated during compression
	 * @param Quality the quality level to compress to
	 * @param bWantColor whether the JPEG is color (true) or monochrome (false)
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before converting, defaults to no scale operation
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 * @param ConvertedImageSize if set, returns the size of the converted image
	 */
	virtual void ConvertToJPEG(CIImage* SourceImage, TArray<uint8>& OutBytes, int32 Quality = 85, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None, FIntPoint* ConvertedImageSize = nullptr) = 0;

#if SUPPORTS_IMAGE_UTILS_2_1
	/**
	 * Converts a image to an array of HEIF data synchronously
	 *
	 * @param SourceImage the image to compress
	 * @param OutBytes the buffer that is populated during compression
	 * @param Quality the quality level to compress to
	 * @param bWantColor whether the HEIF is color (true) or monochrome (false)
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before converting, defaults to no scale operation
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 * @param ConvertedImageSize if set, returns the size of the converted image
	 */
	virtual void ConvertToHEIF(CIImage* SourceImage, TArray<uint8>& OutBytes, int32 Quality = 85,  bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None, FIntPoint* ConvertedImageSize = nullptr) = 0;

	/**
	 * Converts a image to an array of PNG data synchronously
	 *
	 * @param SourceImage the image to compress
	 * @param OutBytes the buffer that is populated during compression
	 * @param Quality the quality level to compress to
	 * @param bWantColor whether the PNG is color (true) or monochrome (false)
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before converting, defaults to no scale operation
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 * @param ConvertedImageSize if set, returns the size of the converted image
	 */
	virtual void ConvertToPNG(CIImage* SourceImage, TArray<uint8>& OutBytes, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None, FIntPoint* ConvertedImageSize = nullptr) = 0;

	/**
	 * Converts a image to an array of TIFF data synchronously
	 *
	 * @param SourceImage the image to compress
	 * @param OutBytes the buffer that is populated during compression
	 * @param Quality the quality level to compress to
	 * @param bUseGpu whether to use the GPU (true) or the CPU (false) to compress
	 * @param Scale whether to scale the image before converting, defaults to no scale operation
	 * @param Rotate a direction to rotate the image in during conversion, defaults to none
	 * @param ConvertedImageSize if set, returns the size of the converted image
	 */
	virtual void ConvertToTIFF(CIImage* SourceImage, TArray<uint8>& OutBytes, bool bWantColor = true, bool bUseGpu = true, float Scale = 1.f, ETextureRotationDirection Rotate = ETextureRotationDirection::None, FIntPoint* ConvertedImageSize = nullptr) = 0;
#endif
#endif
};
