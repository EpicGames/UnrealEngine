// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AppleImageUtilsBlueprintSupport : ModuleRules
	{
		public AppleImageUtilsBlueprintSupport(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
            PublicIncludePaths.AddRange(
                new string[] {
                    // ... add public include paths required here ...
                }
                );


            PrivateIncludePaths.AddRange(
                new string[] {
                    // ... add other private include paths required here ...
                }
                );

			PrivateIncludePathModuleNames.Add("AppleImageUtils");

            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "Engine",
                    "BlueprintGraph",
                    "AppleImageUtils"
                    // ... add other public dependencies that you statically link with here ...
                }
                );

            PublicFrameworks.AddRange(
                new string[]
                {
                    // ... add other public dependencies that you statically link with here ...
                }
                );

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
    				"CoreUObject"
					// ... add private dependencies that you statically link with here ...
				}
				);

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					// ... add any modules that your module loads dynamically here ...
				}
				);
		}
	}
}
