// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_BaseAsyncTask.h"

#include "AppleImageUtilsBlueprintSupport.generated.h"

UCLASS(MinimalAPI)
class UK2Node_ConvertToJPEG :
	public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UK2Node_ConvertToHEIF :
	public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UK2Node_ConvertToTIFF :
	public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI)
class UK2Node_ConvertToPNG :
	public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
};
