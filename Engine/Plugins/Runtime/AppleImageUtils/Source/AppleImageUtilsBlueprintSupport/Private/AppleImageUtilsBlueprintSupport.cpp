// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleImageUtilsBlueprintSupport.h"
#include "AppleImageUtilsBlueprintProxy.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AppleImageUtilsBlueprintSupport)


class FAppleImageUtilsBlueprintSupport :
	public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override {}
	virtual void ShutdownModule() override {}

};

IMPLEMENT_MODULE(FAppleImageUtilsBlueprintSupport, AppleImageUtilsBlueprintSupport);

UK2Node_ConvertToJPEG::UK2Node_ConvertToJPEG(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UAppleImageUtilsBaseAsyncTaskBlueprintProxy, CreateProxyObjectForConvertToJPEG);
	ProxyFactoryClass = UAppleImageUtilsBaseAsyncTaskBlueprintProxy::StaticClass();
	ProxyClass = UAppleImageUtilsBaseAsyncTaskBlueprintProxy::StaticClass();
}

UK2Node_ConvertToHEIF::UK2Node_ConvertToHEIF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UAppleImageUtilsBaseAsyncTaskBlueprintProxy, CreateProxyObjectForConvertToHEIF);
	ProxyFactoryClass = UAppleImageUtilsBaseAsyncTaskBlueprintProxy::StaticClass();
	ProxyClass = UAppleImageUtilsBaseAsyncTaskBlueprintProxy::StaticClass();
}

UK2Node_ConvertToTIFF::UK2Node_ConvertToTIFF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UAppleImageUtilsBaseAsyncTaskBlueprintProxy, CreateProxyObjectForConvertToTIFF);
	ProxyFactoryClass = UAppleImageUtilsBaseAsyncTaskBlueprintProxy::StaticClass();
	ProxyClass = UAppleImageUtilsBaseAsyncTaskBlueprintProxy::StaticClass();
}

UK2Node_ConvertToPNG::UK2Node_ConvertToPNG(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UAppleImageUtilsBaseAsyncTaskBlueprintProxy, CreateProxyObjectForConvertToPNG);
	ProxyFactoryClass = UAppleImageUtilsBaseAsyncTaskBlueprintProxy::StaticClass();
	ProxyClass = UAppleImageUtilsBaseAsyncTaskBlueprintProxy::StaticClass();
}

